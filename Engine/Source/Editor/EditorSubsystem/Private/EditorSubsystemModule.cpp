// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSubsystemModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FEditorSubsystemModule, EditorSubsystem);

void FEditorSubsystemModule::StartupModule()
{

}

void FEditorSubsystemModule::ShutdownModule()
{
}
