// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/Subsystem.h"

#include "EditorSubsystem.generated.h"

/**
 * UEditorSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of the Editor
 *
 * UEditorSubsystems are dynamic and will be initialized when the module is loaded if necessary.
 * This means that after StartupModule() is called on the module containing a subsystem,
 * the subsystem collection with instantiate and initialize the subsystem automatically.
 * If the subsystem collection is created post module load then the instances will be created at
 * collection initialization time.
 */
UCLASS(MinimalAPI, Abstract)
class UEditorSubsystem : public UDynamicSubsystem
{
	GENERATED_BODY()
};

