// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API EDITORSUBSYSTEM_API

class FEditorSubsystemModule : public IModuleInterface
{
public:
	// IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

private:
};

#undef UE_API
